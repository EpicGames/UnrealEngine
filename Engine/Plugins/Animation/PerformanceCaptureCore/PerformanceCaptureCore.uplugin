{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Performance Capture Core",
	"Description": "Performance Capture Core Actor and Component Classes ",
	"Category": "Animation",
	"CreatedBy": "Epic Games",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": false,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "PerformanceCaptureCore",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "PerformanceCaptureCoreEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "IKRig",
			"Enabled": true
		},
		{
			"Name": "LiveLink",
			"Enabled": true
		}
	],
	"SupportURL": ""
}