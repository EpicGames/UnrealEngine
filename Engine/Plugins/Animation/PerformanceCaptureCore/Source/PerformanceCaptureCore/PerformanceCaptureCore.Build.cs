// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PerformanceCaptureCore : ModuleRules
{
	public PerformanceCaptureCore(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
			);
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"IKRig",
				"LiveLinkAnimationCore",
				"LiveLinkInterface"
			}
			);
	}
}
