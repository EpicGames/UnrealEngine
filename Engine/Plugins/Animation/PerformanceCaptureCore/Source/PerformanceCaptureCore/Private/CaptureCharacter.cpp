// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureCharacter.h"

#include "Components/SkeletalMeshComponent.h"
#include "RetargetAnimInstance.h"
#include "RetargetComponent.h"

// Sets default values

#include UE_INLINE_GENERATED_CPP_BY_NAME(CaptureCharacter)
ACaptureCharacter::ACaptureCharacter()
{
	// Set this actor to call Tick() every frame.  You can turn this off to improve performance if you don't need it.
	PrimaryActorTick.bCanEverTick = true;
	RetargetComponent = CreateDefaultSubobject<URetargetComponent>("RetargetComponent");
}


//Update the custom retarget profile
void ACaptureCharacter::SetCustomRetargetProfile(FRetargetProfile InProfile)
{
	RetargetComponent->SetCustomRetargetProfile(InProfile);
}

//Get the custom retarget profile
FRetargetProfile ACaptureCharacter::GetCustomRetargetProfile()
{
	return RetargetComponent->GetCustomRetargetProfile();
}

void ACaptureCharacter::SetSourcePerformer(ACapturePerformer* InPerformer)
{
	RetargetComponent->SetSourcePerformer(InPerformer);
}

//Update the retarget asset
void ACaptureCharacter::SetRetargetAsset(UIKRetargeter* InRetargetAsset)
{
	RetargetComponent->SetRetargetAsset(InRetargetAsset);
}

void ACaptureCharacter::SetForceAllSkeletalMeshesToFollowLeader(bool InFollowLeader)
{
	RetargetComponent->SetForceMeshesFollowLeader(InFollowLeader);
}

URetargetComponent* ACaptureCharacter::GetRetargetComponent() const
{
	return RetargetComponent;
}

void ACaptureCharacter::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	// Fix up deprecation values
PRAGMA_DISABLE_DEPRECATION_WARNINGS

	if (SourcePerformer_DEPRECATED.IsPending())
	{
		RetargetComponent->SourcePerformer = SourcePerformer_DEPRECATED;
		SourcePerformer_DEPRECATED.Reset();
	}

	if (RetargetAsset_DEPRECATED != nullptr)
	{
		RetargetComponent->RetargetAsset = RetargetAsset_DEPRECATED;
		RetargetAsset_DEPRECATED = nullptr;
	}

	// bForceAllSkeletalMeshesToFollowLeader defaults to true. If it is therefore not true, set the false value across
	if (!bForceAllSkeletalMeshesToFollowLeader_DEPRECATED)
	{
		RetargetComponent->bForceOtherMeshesToFollowControlledMesh = false;
		bForceAllSkeletalMeshesToFollowLeader_DEPRECATED = true;
	}

PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
}
