// Copyright Epic Games, Inc. All Rights Reserved.

#include "CapturePerformer.h"
#include "Components/SkeletalMeshComponent.h"
#include "PerformerComponent.h"

// Sets default values

#include UE_INLINE_GENERATED_CPP_BY_NAME(CapturePerformer)
ACapturePerformer::ACapturePerformer()
{
	// Set this actor to call Tick() every frame.  You can turn this off to improve performance if you don't need it.
	PrimaryActorTick.bCanEverTick = true;

	PerformerComponent = CreateDefaultSubobject<UPerformerComponent>("PerformerComponent");
	PerformerComponent->ControlledSkeletalMesh.OverrideComponent = Cast<USkeletalMeshComponent>(GetRootComponent());
}

// Called when the game starts or when spawned
void ACapturePerformer::BeginPlay()
{
	Super::BeginPlay();
}

void ACapturePerformer::PostInitializeComponents()
{
	Super::PostInitializeComponents();
}

void ACapturePerformer::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();
	
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		PerformerComponent->ControlledSkeletalMesh.OverrideComponent = GetSkeletalMeshComponent();
		PerformerComponent->SetLiveLinkSubject(SubjectName);
		PerformerComponent->bForceOtherMeshesToFollowControlledMesh = bForceAllMeshesToFollowLeader;
		PerformerComponent->InitiateAnimation();
	}
}

void ACapturePerformer::OnConstruction(const FTransform& Transform)
{
	Super::OnConstruction(Transform);
}

#if WITH_EDITOR
void ACapturePerformer::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if(PropertyChangedEvent.Property != nullptr)
	{
		if(PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ACapturePerformer, SubjectName))
		{
			PerformerComponent->InitiateAnimation();
			PerformerComponent->SetLiveLinkSubject(SubjectName);
		}
		if(PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ACapturePerformer, bForceAllMeshesToFollowLeader))
		{
			PerformerComponent->bForceOtherMeshesToFollowControlledMesh = bForceAllMeshesToFollowLeader;
			PerformerComponent->InitiateAnimation();
		}
		if(PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ACapturePerformer, bEvaluateAnimation))
		{
			SetEvaluateLiveLinkData(bEvaluateAnimation);
		}
	}
}
#endif

void ACapturePerformer::SetLiveLinkSubject(FLiveLinkSubjectName Subject)
{
	SubjectName = Subject;
	PerformerComponent->SetLiveLinkSubject(Subject);
}

FLiveLinkSubjectName ACapturePerformer::GetLiveLinkSubject() const
{
	return PerformerComponent->SubjectName;
}

void ACapturePerformer::SetEvaluateLiveLinkData(bool bEvaluateLinkLink)
{
	bEvaluateAnimation = bEvaluateLinkLink;
	PerformerComponent->SetEvaluateLiveLinkData(bEvaluateLinkLink);
}

bool ACapturePerformer::GetEvaluateLiveLinkData()
{
	USkeletalMeshComponent* ControlledMesh;
	ControlledMesh = Cast<USkeletalMeshComponent>(PerformerComponent->ControlledSkeletalMesh.GetComponent(GetOwner()));
	ULiveLinkInstance* LiveLinkAnimInstance = Cast<ULiveLinkInstance>(ControlledMesh->GetAnimInstance());
	
	if (IsValid(LiveLinkAnimInstance) && IsValid(ControlledMesh))
	{
		return LiveLinkAnimInstance->GetEnableLiveLinkEvaluation();
	}
	return false;
}

void ACapturePerformer::SetMocapMesh(USkeletalMesh* MocapMesh)
{
	GetSkeletalMeshComponent()->SetSkeletalMeshAsset(MocapMesh);	
}
