// Copyright Epic Games, Inc. All Rights Reserved.

#include "PerformerComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "LiveLinkInstance.h"

// Sets default values for this component's properties

#include UE_INLINE_GENERATED_CPP_BY_NAME(PerformerComponent)
UPerformerComponent::UPerformerComponent()
	:bIsDirty(true)
{
	// Set this component to be initialized when the game starts, and to be ticked every frame. 
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bStartWithTickEnabled = true;
	bTickInEditor = true;
}

void UPerformerComponent::DestroyComponent(bool bPromoteChildren)
{
	//Reset all the skeletal meshes in the owner actor 
	TArray<USkeletalMeshComponent*> SkeletalMeshComponents;
	GetOwner()->GetComponents<USkeletalMeshComponent>(SkeletalMeshComponents);
	for (USkeletalMeshComponent* SkeletalMeshComponent : SkeletalMeshComponents)
	{
		SkeletalMeshComponent->SetUpdateAnimationInEditor(false/* NewUpdateState */);
		SkeletalMeshComponent->SetAnimInstanceClass(nullptr);
		SkeletalMeshComponent->InitAnim(true/* bForceReinit */);
	}
	
	Super::DestroyComponent(bPromoteChildren);
}

void UPerformerComponent::OnRegister()
{
	Super::OnRegister();
	
	if(bIsDirty)
	{
		InitiateAnimation();
	}
}

#if WITH_EDITOR
void UPerformerComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if(PropertyChangedEvent.Property != nullptr)
	{
		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPerformerComponent, SubjectName))
		{
			SetLiveLinkSubject(SubjectName);
			bIsDirty=true;
		}
	
		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPerformerComponent, bEvaluateAnimation))
		{
			SetEvaluateLiveLinkData(bEvaluateAnimation);
		}
	
		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPerformerComponent, ControlledSkeletalMesh))
		{
			bIsDirty=true;
		}
	
		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPerformerComponent, bForceOtherMeshesToFollowControlledMesh))
		{
			bIsDirty=true;
		}
	}
}
#endif

// Called every frame
void UPerformerComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
	
	if(bIsDirty)
	{
		InitiateAnimation();
	}
}

void UPerformerComponent::SetLiveLinkSubject(FLiveLinkSubjectName Subject)
{
	SubjectName = Subject;
	
	USkeletalMeshComponent* ControlledMesh;
	ControlledMesh = Cast<USkeletalMeshComponent>(ControlledSkeletalMesh.GetComponent(GetOwner()));

	if(ControlledMesh)
	{
		ULiveLinkInstance* LiveLinkAnimInstance;
		LiveLinkAnimInstance = Cast<ULiveLinkInstance>(ControlledMesh->GetAnimInstance());
		if(LiveLinkAnimInstance)
		{
			LiveLinkAnimInstance->SetSubject(Subject);
		}
	}
}

FLiveLinkSubjectName UPerformerComponent::GetLiveLinkSubject() const
{
	return SubjectName;
}

void UPerformerComponent::SetEvaluateLiveLinkData(bool bEvaluateLinkLink)
{
	bEvaluateAnimation = bEvaluateLinkLink;
	
	USkeletalMeshComponent* ControlledMesh;
	ControlledMesh = Cast<USkeletalMeshComponent>(ControlledSkeletalMesh.GetComponent(GetOwner()));

	if(ControlledMesh)
	{
		ULiveLinkInstance* LiveLinkAnimInstance;
		LiveLinkAnimInstance = Cast<ULiveLinkInstance>(ControlledMesh->GetAnimInstance());
	
		if (LiveLinkAnimInstance)
		{
			LiveLinkAnimInstance->EnableLiveLinkEvaluation(bEvaluateLinkLink);
		}
	}
}

bool UPerformerComponent::GetEvaluateLiveLinkData() const
{
	return bEvaluateAnimation;
}

bool UPerformerComponent::SetControlledMesh(USkeletalMeshComponent* SkeletalMesh)
{
	if (SkeletalMesh)
	{
		if (SkeletalMesh->GetOwner() == this->GetOwner()) //Check the given skeletal mesh component belongs to the same actor as this performer component and is valid
		{
			ControlledSkeletalMesh.OverrideComponent = SkeletalMesh;
			bIsDirty=true; // Mark bIsDirty true so that next frame InitiateAnimation is called.
			return true;
		}
	}

	return false;
}

void UPerformerComponent::SetForceMeshesFollowLeader(bool bInBool)
{
	bForceOtherMeshesToFollowControlledMesh = bInBool;
	bIsDirty=true; // Mark bIsDirty true so that next frame InitiateAnimation is called.
}

void UPerformerComponent::InitiateAnimation()
{
	TArray<USkeletalMeshComponent*> OwnerSkeletalMeshComponents;
	GetOwner()->GetComponents<USkeletalMeshComponent>(OwnerSkeletalMeshComponents);

	//Set the mesh that will be controlled
	USkeletalMeshComponent* ControlledMesh = Cast<USkeletalMeshComponent>(ControlledSkeletalMesh.GetComponent(GetOwner()));
	
	//If there is no controlled mesh, reset everything
	if(!IsValid(ControlledMesh))
	{
		for (USkeletalMeshComponent* OwnerSkeletalMeshComponent : OwnerSkeletalMeshComponents)
		{
			OwnerSkeletalMeshComponent->SetAnimInstanceClass(nullptr);
			OwnerSkeletalMeshComponent->InitAnim(true /*bForceReinit*/);
			OwnerSkeletalMeshComponent->SetUpdateAnimationInEditor(true);
		}
	}

	//Reset all skeletal meshes on owner
	for (USkeletalMeshComponent* OwnerSkeletalMeshComponent : OwnerSkeletalMeshComponents)
	{
		//Null out the given leader pose to avoid a circular dependency.
		if (bForceOtherMeshesToFollowControlledMesh)
		{
			OwnerSkeletalMeshComponent->SetLeaderPoseComponent(nullptr);
		}
		OwnerSkeletalMeshComponent->InitAnim(true /*bForceReinit*/);
		
		//Skip any skeletal meshes that already have anim instance set that's not a LiveLinkInstance
		if(OwnerSkeletalMeshComponent->GetAnimClass()==ULiveLinkInstance::StaticClass())
		{
			OwnerSkeletalMeshComponent->SetAnimInstanceClass(nullptr);
			OwnerSkeletalMeshComponent->InitAnim(true /*bForceReinit*/);
			OwnerSkeletalMeshComponent->SetUpdateAnimationInEditor(true);
		}
	}
	//Set the chosen mesh as leader
	for (USkeletalMeshComponent* OwnerSkeletalMeshComponent : OwnerSkeletalMeshComponents)
	{
		if(IsValid(OwnerSkeletalMeshComponent))
		{
			if(OwnerSkeletalMeshComponent!=ControlledMesh && bForceOtherMeshesToFollowControlledMesh)
			{
				OwnerSkeletalMeshComponent->SetLeaderPoseComponent(ControlledMesh);
			}
		}
	}

	if(IsValid(ControlledMesh))
	{
		ControlledMesh->SetAnimInstanceClass(ULiveLinkInstance::StaticClass());
		ControlledMesh->InitAnim(true /*bForceReinit*/);
		ControlledMesh->SetUpdateAnimationInEditor(true);
		ControlledMesh->bPropagateCurvesToFollowers = true;
		ControlledMesh->VisibilityBasedAnimTickOption = EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones;
		SetLiveLinkSubject(SubjectName);
		SetEvaluateLiveLinkData(bEvaluateAnimation);
	}	
	bIsDirty=false;
}
