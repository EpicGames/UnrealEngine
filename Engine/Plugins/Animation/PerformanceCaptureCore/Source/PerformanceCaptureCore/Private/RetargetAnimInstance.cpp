// Copyright Epic Games, Inc. All Rights Reserved.

#include "RetargetAnimInstance.h"
#include "RetargetInstanceProxy.h"

///////////////////////////////////
/// Anim Instance 
///////////////////////////////////

#include UE_INLINE_GENERATED_CPP_BY_NAME(RetargetAnimInstance)

void URetargetAnimInstance::UpdateCustomRetargetProfile(const FRetargetProfile &InRetargetProfile)
{
	FRetargetAnimInstanceProxy& Proxy = GetProxyOnGameThread<FRetargetAnimInstanceProxy>();
	Proxy.RetargetNode->CustomRetargetProfile = InRetargetProfile;
}

FRetargetProfile URetargetAnimInstance::GetRetargetProfile()
{
	FRetargetAnimInstanceProxy& Proxy = GetProxyOnGameThread<FRetargetAnimInstanceProxy>();
	FRetargetProfile OutProfile = Proxy.RetargetNode->CustomRetargetProfile;
	return OutProfile;
}

void URetargetAnimInstance::NativeInitializeAnimation()
{
	FRetargetAnimInstanceProxy& Proxy = GetProxyOnGameThread<FRetargetAnimInstanceProxy>();
	Proxy.Initialize(this);
}

void URetargetAnimInstance::ConfigureAnimInstance(UIKRetargeter* InIKRetargetAsset, TWeakObjectPtr<USkeletalMeshComponent> InSourceMeshComponent, FRetargetProfile InRetargetProfile)
{
	FRetargetAnimInstanceProxy& Proxy = GetProxyOnGameThread<FRetargetAnimInstanceProxy>();
	Proxy.ConfigureAnimInstanceProxy(InIKRetargetAsset, InSourceMeshComponent, InRetargetProfile);
}

FAnimInstanceProxy* URetargetAnimInstance::CreateAnimInstanceProxy()
{
	return new FRetargetAnimInstanceProxy(this, &RetargetNode);
}
