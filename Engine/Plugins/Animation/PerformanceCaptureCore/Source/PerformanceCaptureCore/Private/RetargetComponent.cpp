// Copyright Epic Games, Inc. All Rights Reserved.

#include "RetargetComponent.h"
#include "CapturePerformer.h"
#include "Components/SkeletalMeshComponent.h"
#include "LiveLinkInstance.h"
#include "RetargetAnimInstance.h"

// Sets default values for this component's properties

#include UE_INLINE_GENERATED_CPP_BY_NAME(RetargetComponent)
URetargetComponent::URetargetComponent()
{
	// Set this component to be initialized when the game starts, and to be ticked every frame. 
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bStartWithTickEnabled = true;
	bTickInEditor = true;
}

void URetargetComponent::SetCustomRetargetProfile(FRetargetProfile InProfile)
{
	CustomRetargetProfile = InProfile;

	const TObjectPtr<USkeletalMeshComponent> ControlledMesh = Cast<USkeletalMeshComponent>(ControlledSkeletalMeshComponent.GetComponent(GetOwner()));
	if(ControlledMesh)
	{
		if(ControlledMesh->GetAnimInstance())
		{
			if (URetargetAnimInstance* AnimInstance = Cast<URetargetAnimInstance>(ControlledMesh->GetAnimInstance()))
			{
				AnimInstance->UpdateCustomRetargetProfile(CustomRetargetProfile);
			}
		}
	}
	bIsDirty = true;
}

FRetargetProfile URetargetComponent::GetCustomRetargetProfile()
{
	/// Check for valid components and refs and then get the custom retarget profile struct
	FRetargetProfile OutProfile;
	if(TObjectPtr<USkeletalMeshComponent> ControlledMesh = Cast<USkeletalMeshComponent>(ControlledSkeletalMeshComponent.GetComponent(GetOwner())))
	{
		if(ControlledMesh->GetAnimInstance())
		{
			if(URetargetAnimInstance* AnimInstance = Cast<URetargetAnimInstance>(ControlledMesh->GetAnimInstance()))
			{
				OutProfile = AnimInstance->GetRetargetProfile();
			}
		}
	}
	return OutProfile;
}

void URetargetComponent::SetForceMeshesFollowLeader(const bool bInBool)
{
	SetForceOtherMeshesToFollowControlledMesh(bInBool);
}

void URetargetComponent::SetSourcePerformer(ACapturePerformer* InPerformer)
{
	if (IsValid(InPerformer))
	{
		SourcePerformer = InPerformer;
		SetSourcePerformerMesh(InPerformer->GetSkeletalMeshComponent());
	}
	else
	{
		SourcePerformer = nullptr;
		SetSourcePerformerMesh(Cast<USkeletalMeshComponent>(SourceSkeletalMeshComponent.GetComponent(GetOwner())));
	}
	bIsDirty = true;
}

void URetargetComponent::SetSourcePerformerMesh(USkeletalMeshComponent* InPerformerMesh)
{
	SourceSkeletalMeshComponent.OverrideComponent = InPerformerMesh;
	bIsDirty = true;
}


void URetargetComponent::SetControlledMesh(USkeletalMeshComponent* InControlledMesh)
{
	ControlledSkeletalMeshComponent.OverrideComponent = InControlledMesh;
	bIsDirty = true;
}

void URetargetComponent::SetRetargetAsset(UIKRetargeter* InRetargetAsset)
{
	RetargetAsset = InRetargetAsset; //It is possible to pass a nullptr to the AnimInstance 

	/// Check for valid components and refs and then reint animation to reset pose
	if(TObjectPtr<USkeletalMeshComponent> ControlledMesh = Cast<USkeletalMeshComponent>(ControlledSkeletalMeshComponent.GetComponent(GetOwner())))
	{
		if(ControlledMesh->GetAnimInstance())
		{
			URetargetAnimInstance* AnimInstance = Cast<URetargetAnimInstance>(ControlledMesh->GetAnimInstance());

			const TObjectPtr<USkeletalMeshComponent> SourceMesh = Cast<USkeletalMeshComponent>(SourceSkeletalMeshComponent.GetComponent(GetOwner()));
		
			if(SourceMesh && AnimInstance)
			{
				AnimInstance->ConfigureAnimInstance(RetargetAsset, SourceMesh, CustomRetargetProfile);
				SourceMesh->InitAnim(true /*bForceReinit*/);
			}
			ControlledMesh->InitAnim(true /*bForceReinit*/);
		}
	}
	bIsDirty = true;
}

void URetargetComponent::OnRegister()
{
	Super::OnRegister();

	SourceSkeletalMeshComponent.OverrideComponent = nullptr;

	USkeletalMeshComponent* ControlledMesh = Cast<USkeletalMeshComponent> (ControlledSkeletalMeshComponent.GetComponent(GetOwner()));

	if (ControlledMesh)
	{
		const TObjectPtr<URetargetAnimInstance> AnimInstance = Cast<URetargetAnimInstance>(ControlledMesh->GetAnimInstance());

		if(AnimInstance) //Only set these properties if we have a valid AnimInstance of the correct class.
		{
			SetForceMeshesFollowLeader(bForceOtherMeshesToFollowControlledMesh);
			SetCustomRetargetProfile(CustomRetargetProfile);
		}
	}

	if (!GetOwner()->HasAnyFlags(RF_ClassDefaultObject))
	{
		SetControlledMesh(ControlledMesh);
		SetSourcePerformer(SourcePerformer.Get());
		SetRetargetAsset(RetargetAsset);
		SetForceMeshesFollowLeader(bForceOtherMeshesToFollowControlledMesh);
	}
}

void URetargetComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
	if(bIsDirty)
	{
		InitiateAnimation();
	}
}

void URetargetComponent::DestroyComponent(bool bPromoteChildren)
{
	//When the component is removed, reset the state of the skelmeshes in the owner actor
	TArray<USkeletalMeshComponent*> OwnerSkeletalMeshComponents;
	GetOwner()->GetComponents<USkeletalMeshComponent>(OwnerSkeletalMeshComponents);
	
	for (USkeletalMeshComponent* OwnerSkeletalMeshComponent : OwnerSkeletalMeshComponents)
	{
		//If the anim class is LiveLinkInstance assume user does not want to reset that skeletal mesh component
		if(OwnerSkeletalMeshComponent->GetAnimClass()!=ULiveLinkInstance::StaticClass())
		{
			OwnerSkeletalMeshComponent->SetAnimInstanceClass(nullptr);
			OwnerSkeletalMeshComponent->SetLeaderPoseComponent(nullptr);
			OwnerSkeletalMeshComponent->InitAnim(true /*bForceReinit*/);
			OwnerSkeletalMeshComponent->SetUpdateAnimationInEditor(false);
		}
	}
	Super::DestroyComponent(bPromoteChildren);
}

#if WITH_EDITOR
void URetargetComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	if(PropertyChangedEvent.Property != nullptr)
	{
		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(URetargetComponent, SourceSkeletalMeshComponent))
		{
			// OtherActor is set from the property change and should be used so we want to clear the override.
			SetSourcePerformerMesh(nullptr);

			// If the change was to zero out the OtherActor as well, we want to now force the override to self.
			if (SourceSkeletalMeshComponent.OverrideComponent == nullptr && SourceSkeletalMeshComponent.OtherActor == nullptr)
			{
				SourceSkeletalMeshComponent.OverrideComponent = SourceSkeletalMeshComponent.GetComponent(GetOwner());
			}
		}

		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(URetargetComponent, ControlledSkeletalMeshComponent))
		{
			// OtherActor is set from the property change and should be used so we want to clear the override.
			SetControlledMesh(nullptr);

			// If the change was to zero out the OtherActor as well, we want to now force the override to self.
			if (ControlledSkeletalMeshComponent.OverrideComponent == nullptr && ControlledSkeletalMeshComponent.OtherActor == nullptr)
			{
				ControlledSkeletalMeshComponent.OverrideComponent = ControlledSkeletalMeshComponent.GetComponent(GetOwner());
			}
		}

		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(URetargetComponent, bForceOtherMeshesToFollowControlledMesh))
		{
			SetForceOtherMeshesToFollowControlledMesh(bForceOtherMeshesToFollowControlledMesh);
		}

		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(URetargetComponent, RetargetAsset))
		{
			SetRetargetAsset(RetargetAsset);
		}
		
		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(URetargetComponent, CustomRetargetProfile))
		{
			SetCustomRetargetProfile(CustomRetargetProfile);
		}

		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(URetargetComponent, SourcePerformer))
		{
			SetSourcePerformer(SourcePerformer.Get());
		}
	}
}
#endif

void URetargetComponent::SetForceOtherMeshesToFollowControlledMesh(bool bInBool)
{
	bForceOtherMeshesToFollowControlledMesh = bInBool;
	
	USkeletalMeshComponent* ControlledMesh = Cast<USkeletalMeshComponent> (ControlledSkeletalMeshComponent.GetComponent(GetOwner()));
	const USkeletalMeshComponent* SourceMesh = Cast<USkeletalMeshComponent>(SourceSkeletalMeshComponent.GetComponent(GetOwner()));
	
	TArray<USkeletalMeshComponent*> OwnerSkeletalMeshComponents;
	//Check the owner is valid because it can be Null if being reconstructed in the BP editor.
	if(GetOwner()) 
	{
		GetOwner()->GetComponents<USkeletalMeshComponent>(OwnerSkeletalMeshComponents);
	}
	
	if(IsValid(ControlledMesh) && IsValid(SourceMesh))
	{
		for (USkeletalMeshComponent* OwnerSkeletalMeshComponent : OwnerSkeletalMeshComponents)
		{
			if(IsValid(OwnerSkeletalMeshComponent))
			{
				const bool bShouldResetMesh = OwnerSkeletalMeshComponent->GetAnimClass()!=ULiveLinkInstance::StaticClass()
				&& OwnerSkeletalMeshComponent!=ControlledMesh && OwnerSkeletalMeshComponent!=SourceMesh;
				if(bShouldResetMesh)
				{
					OwnerSkeletalMeshComponent->SetAnimInstanceClass(nullptr);
					OwnerSkeletalMeshComponent->InitAnim(true /*bForceReinit*/);
					OwnerSkeletalMeshComponent->SetUpdateAnimationInEditor(true);
					OwnerSkeletalMeshComponent->SetLeaderPoseComponent(nullptr);
				}	
			}
		}
		
		if(bForceOtherMeshesToFollowControlledMesh)
		{
			//Set all skeletal meshes to tick in editor
			for (USkeletalMeshComponent* OwnerSkeletalMeshComponent : OwnerSkeletalMeshComponents)
			{
				OwnerSkeletalMeshComponent->SetUpdateAnimationInEditor(bInBool);

				///If not Controlled or the Source mesh then set skeletal meshes to follow the ControlledMesh. Exception for LiveLink instances as we assume those are being driven directly be mocap data using the PCapPerformerComponent
				///
				const bool bShouldSetLeaderPose  = OwnerSkeletalMeshComponent!=ControlledMesh && OwnerSkeletalMeshComponent->GetAnimClass()!=ULiveLinkInstance::StaticClass() && OwnerSkeletalMeshComponent!=SourceMesh;
				if(bShouldSetLeaderPose)
				{
					OwnerSkeletalMeshComponent->SetLeaderPoseComponent(ControlledMesh,true /*ForceUpdate*/, true /*FollowerShouldTickPose*/);
				}
			}
		}
	}
	bIsDirty = true;
}

void URetargetComponent::InitiateAnimation()
{
	USkeletalMeshComponent* ControlledMesh = Cast<USkeletalMeshComponent> (ControlledSkeletalMeshComponent.GetComponent(GetOwner()));
	USkeletalMeshComponent* SourceMesh = Cast<USkeletalMeshComponent>(SourceSkeletalMeshComponent.GetComponent(GetOwner()));
	
	const bool bShouldBeReinitialized = IsValid(RetargetAsset) && IsValid(ControlledMesh);
	
	if(bShouldBeReinitialized)
	{
		//Set the anim instance class on the controlled mesh to use RetargetAnimInstance
		ControlledMesh->SetAnimInstanceClass(URetargetAnimInstance::StaticClass());

		TObjectPtr<URetargetAnimInstance> AnimInstance = Cast<URetargetAnimInstance>(ControlledMesh->GetAnimInstance());

		if (SourceMesh && SourceMesh->GetOwner() && SourceMesh!=ControlledMesh)
		{
			ControlledMesh->AddTickPrerequisiteComponent(SourceMesh);
		}
		
		if(AnimInstance)
		{
			RetargetAsset->IncrementVersion();
			AnimInstance->ConfigureAnimInstance(RetargetAsset, SourceMesh, CustomRetargetProfile);
			ControlledMesh->SetUpdateAnimationInEditor(true);
			ControlledMesh->bPropagateCurvesToFollowers = true;
			ControlledMesh->VisibilityBasedAnimTickOption = EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones;
			ControlledMesh->InitAnim(true /*bForceReinit*/);
		}
		ControlledMesh->InitAnim(true /*bForceReinit*/);
	}
	bIsDirty=false;
}
