// Copyright Epic Games, Inc. All Rights Reserved.
#include "RetargetInstanceProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RetargetInstanceProxy)


/////////////////////////////////
/// Anim instance proxy struct
/////////////////////////////////

FRetargetAnimInstanceProxy::FRetargetAnimInstanceProxy(UAnimInstance* InAnimInstance, FAnimNode_RetargetPoseFromMesh* InRetargetNode)
	: FAnimInstanceProxy(InAnimInstance),
	RetargetNode(InRetargetNode)
{
}

void FRetargetAnimInstanceProxy::Initialize(UAnimInstance* InAnimInstance)
{
	FAnimInstanceProxy::Initialize(InAnimInstance);

	//Initialize instance manually
	FAnimationInitializeContext InitContext(this);
	RetargetNode->Initialize_AnyThread(InitContext);
}

void FRetargetAnimInstanceProxy::PreUpdate(UAnimInstance* InAnimInstance, float DeltaSeconds)
{
	if(InAnimInstance)
	{
		Super::PreUpdate(InAnimInstance, DeltaSeconds);
				
		if (RetargetNode->HasPreUpdate())
		{
			RetargetNode->PreUpdate(InAnimInstance);
		}
	}
}

// Cache bones if they are marked as invalid//
void FRetargetAnimInstanceProxy::CacheBones()
{
	if(bBoneCachesInvalidated)
	{
		FAnimationCacheBonesContext Context(this);
		RetargetNode->CacheBones_AnyThread(Context);
		bBoneCachesInvalidated = false;
	}
}

// Evaluate pose//
bool FRetargetAnimInstanceProxy::Evaluate(FPoseContext& Output)
{
	Super::Evaluate(Output);
	
	/* This evaluates UAnimNode_RetargetPoseFromMesh */
	RetargetNode->Evaluate_AnyThread(Output);
	return true;
}

//Update animation//
void FRetargetAnimInstanceProxy::UpdateAnimationNode(const FAnimationUpdateContext& InContext)
{
	Super::UpdateAnimationNode(InContext);

	UpdateCounter.Increment();
	
	RetargetNode->Update_AnyThread(InContext);
}

void FRetargetAnimInstanceProxy::ConfigureAnimInstanceProxy(UIKRetargeter* InIKRetargetAsset, TWeakObjectPtr<USkeletalMeshComponent> InSourceMeshComponent, FRetargetProfile &InRetargetProfile)
{
	RetargetNode->IKRetargeterAsset = InIKRetargetAsset;
	RetargetNode->RetargetFrom = ERetargetSourceMode::CustomSkeletalMeshComponent;
	RetargetNode->SourceMeshComponent = InSourceMeshComponent;
	RetargetNode->CustomRetargetProfile = InRetargetProfile;
	
	if (FIKRetargetProcessor* Processor = RetargetNode->GetRetargetProcessor())
	{
		Processor->SetNeedsInitialized();
	}
}
