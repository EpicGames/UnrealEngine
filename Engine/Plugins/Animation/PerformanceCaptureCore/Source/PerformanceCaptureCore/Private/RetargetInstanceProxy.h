// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RetargetAnimInstance.h"
#include "Animation/AnimInstanceProxy.h"

#include "RetargetInstanceProxy.generated.h"

class UIKRetargeter;
class USkeletalMeshComponent;
struct FAnimNode_RetargetPoseFromMesh;

////////////////////////////////////
/// Anim Instance Proxy
////////////////////////////////////
USTRUCT()
struct FRetargetAnimInstanceProxy : public FAnimInstanceProxy
{
public:
	
	GENERATED_BODY()

	FRetargetAnimInstanceProxy() = default; //Constructor
	
	FRetargetAnimInstanceProxy(UAnimInstance* InAnimInstance, FAnimNode_RetargetPoseFromMesh* InRetargetNode);
	
public:	
	/* FAnimInstanceProxy Instance */ 
	virtual void Initialize(UAnimInstance* InAnimInstance) override; 
	virtual void PreUpdate(UAnimInstance* InAnimInstance, float DeltaSeconds) override;
	virtual void CacheBones() override;
	virtual bool Evaluate(FPoseContext& Output) override;
	virtual void UpdateAnimationNode(const FAnimationUpdateContext& InContext) override;
	/* END FAnimInstanceProxy Instance */
	
	void ConfigureAnimInstanceProxy(UIKRetargeter* InIKRetargetAsset, TWeakObjectPtr<USkeletalMeshComponent> InSourceMeshComponent,FRetargetProfile &InRetargetProfile);

	FAnimNode_RetargetPoseFromMesh* RetargetNode = nullptr;

private:	
	FRetargetProfile* RetargetProfile = nullptr;
	
};
