// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Animation/SkeletalMeshActor.h"
#include "CapturePerformer.h"
#include "Retargeter/IKRetargetProfile.h"

#include "CaptureCharacter.generated.h"

#define UE_API PERFORMANCECAPTURECORE_API

class URetargetComponent;
class UIKRetargeter;

UCLASS(MinimalAPI, Blueprintable, ClassGroup ="Performance Capture", Category = "Performance Capture",  HideCategories = ("Mesh", "Rendering", "Animation", "LOD", "Misc", "Physics", "Streaming"))
class ACaptureCharacter : public ASkeletalMeshActor
{
	GENERATED_BODY()

public:
	// Sets default values for this actor's properties
	UE_API ACaptureCharacter();

#if WITH_EDITORONLY_DATA
	/**
	* CapturePerformer Actor that will be the source for retargeting.
	*/
	UE_DEPRECATED(5.7, "Use URetargetComponent::SourcePerformer instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use URetargetComponent::SourcePerformer instead."))
	TSoftObjectPtr<ACapturePerformer> SourcePerformer_DEPRECATED;

	/**
	* The IKRetarget Asset to use for retargeting between the SourcePerformer and this Character.
	*/
	UE_DEPRECATED(5.7, "Use URetargetComponent::RetargetAsset instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use URetargetComponent::RetargetAsset instead."))
	TObjectPtr<UIKRetargeter> RetargetAsset_DEPRECATED;

	/**
	* Force all skeletal meshes to use the root SkeletalMesh as their Leader. Default = True.
	*/
	UE_DEPRECATED(5.7, "Use URetargetComponent::bForceOtherMeshesToFollowControlledMesh instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use URetargetComponent::bForceOtherMeshesToFollowControlledMesh instead."))
	bool bForceAllSkeletalMeshesToFollowLeader_DEPRECATED = true;
#endif // WITH_EDITORONLY_DATA

	/**
	* Set a Custom Retarget Profile.
	* @param InProfile New Retarget Profile.
	*/
	UFUNCTION(BlueprintCallable, Category = "Performance Capture|Character")
	UE_API void SetCustomRetargetProfile(FRetargetProfile InProfile);

	/**
	* Get a Custom Retarget Profile.
	* @return Current Custom Retarget Profile.
	*/
	UFUNCTION(BlueprintCallable, Category = "Performance Capture|Character")
	UE_API FRetargetProfile GetCustomRetargetProfile();

	/**
	* Set the Source CapturePerformer Actor.
	* @param InPerformer New Performer.
	*/
	UFUNCTION(BlueprintCallable, Category = "Performance Capture|Character")
	UE_API void SetSourcePerformer(ACapturePerformer* InPerformer);

	/**
	* Set the Retarget Asset.
	* @param InRetargetAsset New IKRetarget Asset.
	*/
	UFUNCTION(BlueprintCallable, Category = "Performance Capture|Character")
	UE_API void SetRetargetAsset(UIKRetargeter* InRetargetAsset);

	/**
	* Force all Skeletal Meshes to follow the Controlled Skeletal mesh. 
	* @param InFollowLeader New Bool.
	*/
	UFUNCTION(BlueprintCallable, Category = "Performance Capture|Character")
	UE_API void SetForceAllSkeletalMeshesToFollowLeader (bool InFollowLeader);

	/**
	* Return the retarget component
	*/
	UFUNCTION(BlueprintPure, Category = "Performance Capture|Character")
	UE_API URetargetComponent* GetRetargetComponent() const;

protected:
	// UObject interface
	virtual void PostLoad() override;
	// End of UObject interface

private:
	// retarget component is private and hidden from the editor UI
	UPROPERTY()
	TObjectPtr<URetargetComponent> RetargetComponent;
};

#undef UE_API
