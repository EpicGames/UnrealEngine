// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/SkeletalMeshActor.h"
#include "LiveLinkTypes.h"
#include "LiveLinkInstance.h"
#include "CapturePerformer.generated.h"

#define UE_API PERFORMANCECAPTURECORE_API

class ULiveLinkInstance;
class UPerformerComponent;

UCLASS(MinimalAPI, Blueprintable, Category="Performance Capture")
class ACapturePerformer : public ASkeletalMeshActor
{
	GENERATED_BODY()

public:
	// Sets default values for this actor's properties
	UE_API ACapturePerformer();

	/**
	* LiveLink Subject Name. Must have the Animation Role Type.
	*/
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Performance Capture")
	FLiveLinkSubjectName SubjectName;

	/**
	* Force all skeletal meshes to follow the pose of the Controlled Skeletal Mesh.
	*/
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Performance Capture")
	bool bForceAllMeshesToFollowLeader = true;

	/**
	* Evaluate LiveLink animation. Set to false to pause animation.
	*/
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Performance Capture")
	bool bEvaluateAnimation = true;

	/**
	* Set the LiveLink Subject Name. Subject must have the Animation Role Type.
	* @param Subject New LiveLink Subject.
	*/
	UFUNCTION(BlueprintCallable, Category = "Performance Capture|Performer")
	UE_API void SetLiveLinkSubject(FLiveLinkSubjectName Subject);

	/**
	* Get the LiveLink Subject Name.
	* @return FLiveLinkSubjectName Current LiveLink Subject.
	*/
	UFUNCTION(BlueprintCallable, Category ="Performance Capture|Performer")
	UE_API FLiveLinkSubjectName GetLiveLinkSubject() const;

	/**
	* Set the LiveLink data to update the Skeletal Mesh pose.
	* @param bEvaluateLinkLink Drive or pause the Skeletal Mesh from LiveLink Subject data.
	*/
	UFUNCTION(BlueprintCallable, Category ="Performance Capture|Performer")
	UE_API void SetEvaluateLiveLinkData(bool bEvaluateLinkLink);

	/**
	* Get state of LiveLink data evaluation. True = animation is updated.
	* @return bool Is LiveLink data being evaluated.
	*/
	UFUNCTION(BlueprintPure, Category ="Performance Capture|Performer")
	UE_API bool GetEvaluateLiveLinkData();

	/**
	* Set the Skeletal Mesh Asset of the root Skeletal Mesh.
	* @param MocapMesh New Skeletal Mesh Asset.
	*/
	UFUNCTION(BlueprintCallable, Category = "Performance Capture|Performer")
	UE_API void SetMocapMesh(USkeletalMesh* MocapMesh);

protected:
	// Called when the game starts or when spawned
	UE_API virtual void BeginPlay() override;
	UE_API virtual void PostInitializeComponents() override;
	UE_API virtual void PostRegisterAllComponents() override;
	UE_API virtual void OnConstruction(const FTransform& Transform) override;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	
private:
	TObjectPtr<UPerformerComponent> PerformerComponent;
};

#undef UE_API
