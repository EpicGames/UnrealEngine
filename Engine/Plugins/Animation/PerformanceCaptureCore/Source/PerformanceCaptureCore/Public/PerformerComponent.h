// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LiveLinkTypes.h"
#include "Components/ActorComponent.h"

#include "PerformerComponent.generated.h"

#define UE_API PERFORMANCECAPTURECORE_API

class ULiveLinkInstance;

UCLASS(MinimalAPI, BlueprintType, DisplayName="Performer Component",  ClassGroup=("Performance Capture"), meta=(BlueprintSpawnableComponent))
class UPerformerComponent : public UActorComponent
{
	GENERATED_BODY()

public:
	// Sets default values for this component's properties
	UE_API UPerformerComponent();

	/**
	* LiveLink Subject Name. Must have the Animation Role Type.
	*/
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Performance Capture")
	FLiveLinkSubjectName SubjectName;

	/**
	* The Skeletal Mesh driven by the LiveLink subject. Skeleton mush be compatible with the LiveLink subject's bone hierarchy. 
	*/
	UPROPERTY(BlueprintReadOnly, EditInstanceOnly, Category = "Performance Capture", meta=(UseComponentPicker, AllowedClasses = "/Script/Engine.SkeletalMeshComponent"))
	FComponentReference ControlledSkeletalMesh;
	
	/**
	* Evaluate LiveLink animation. Set to false to pause animation.
	*/
	UPROPERTY(EditAnywhere, Category = "Performance Capture")
	bool bEvaluateAnimation = true;
	
	/**
	* Force all other skeletal meshes in the Owner Actor to follow the pose of the Controlled Skeletal Mesh.
	*/
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Performance Capture")
	bool bForceOtherMeshesToFollowControlledMesh = true;

	/**
	* Set the LiveLink Subject Name. Subject must have the Animation Role Type.
	* @param Subject New LiveLink Subject.
	*/
	UFUNCTION(BlueprintCallable, Category = "Performance Capture|Performer")
	UE_API void SetLiveLinkSubject(FLiveLinkSubjectName Subject);
	
	/**
	* Get the LiveLink Subject Name.
	* @return FLiveLinkSubjectName Current LiveLink Subject.
	*/
	UFUNCTION(BlueprintCallable, Category ="Performance Capture|Performer")
	UE_API FLiveLinkSubjectName GetLiveLinkSubject() const;
	
	/**
	* Set the LiveLink data to update the Skeletal Mesh pose.
	* @param bEvaluateLinkLink Drive or pause the Skeletal Mesh from LiveLink Subject data.
	*/
	UFUNCTION(BlueprintCallable, Category ="Performance Capture|Performer")
	UE_API void SetEvaluateLiveLinkData(bool bEvaluateLinkLink);
	
	/**
	* Get the LiveLink Evaluation State. Subject must have the Animation Role Type.
	* @return bool Is LiveLink data being evaluated.
	*/
	UFUNCTION(BlueprintPure, Category ="Performance Capture|Performer")
	UE_API bool GetEvaluateLiveLinkData() const;

	/**
	* Set the controlled skeletal mesh component. Must be a member of the owner actor of this component.
	* @param SkeletalMesh New SkeletalMesh.
	* @return bool Was the given skeletal mesh component successfully applied.
	*/
	UFUNCTION(BlueprintCallable, Category = "Performance Capture|Performer")
	UE_API bool SetControlledMesh(USkeletalMeshComponent* SkeletalMesh);

	/**
	* Set the bForceOtherMeshesToFollowControlledMesh variable. Changing will re-initialise animation.
	* @param bInBool DForce all other skeletal meshes in the Owner Actor to follow the pose of the Controlled Skeletal Mesh.
	*/
	UFUNCTION(BlueprintCallable, Category ="Performance Capture|Performer")
	UE_API void SetForceMeshesFollowLeader(bool bInBool);

protected:
	UE_API virtual void DestroyComponent(bool bPromoteChildren) override;

	UE_API virtual void OnRegister() override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

public:
	// Called every frame
	UE_API virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	UE_API void InitiateAnimation();

private:

	bool bIsDirty;
	
};

#undef UE_API
