// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimInstance.h"
#include "Animation/AnimNodeBase.h"
#include "AnimNodes/AnimNode_RetargetPoseFromMesh.h"

#include "RetargetAnimInstance.generated.h"

class UIKRetargeter;
class USkeletalMeshComponent;
struct FAnimNode_RetargetPoseFromMesh;
struct FRetargetAnimInstanceProxy;

///////////////////////////
/// Anim Instance 
///////////////////////////

UCLASS(Transient, NotBlueprintable, BlueprintType, MinimalAPI)
class URetargetAnimInstance : public UAnimInstance
{
	GENERATED_BODY()

	friend FRetargetAnimInstanceProxy;

public:
	/**
	* Configure Retarget AnimInstance
	* @param InIKRetargetAsset IKRetargeter Asset to use for retargeting.
	* @param InSourceMeshComponent The mesh component that is the source of motion.
	* @param InRetargetProfile Retarget Profile struct to use.
	*/
	void ConfigureAnimInstance(UIKRetargeter* InIKRetargetAsset,TWeakObjectPtr<USkeletalMeshComponent> InSourceMeshComponent, FRetargetProfile InRetargetProfile);
	
	/**
	* Update the RetargetProfile.
	* @param InRetargetProfile Retarget Profile struct to use.
	*/
	void UpdateCustomRetargetProfile(const FRetargetProfile &InRetargetProfile);

	/**
	* Retrieve the current RetargetProfile struct.
	* @return  Retarget Profile struct to use.
	*/
	FRetargetProfile GetRetargetProfile();

protected:
	/** UAnimInstance interface */
	virtual void NativeInitializeAnimation() override;
	virtual FAnimInstanceProxy* CreateAnimInstanceProxy() override;
	/** UAnimInstance interface end*/
	
	UPROPERTY()
	FAnimNode_RetargetPoseFromMesh RetargetNode;
};
