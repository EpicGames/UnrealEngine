// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/ActorComponent.h"
#include "Retargeter/IKRetargetProfile.h"

#include "RetargetComponent.generated.h"

#define UE_API PERFORMANCECAPTURECORE_API

class ACapturePerformer;
class UIKRetargeter;

UCLASS(MinimalAPI, BlueprintType, ClassGroup=("Performance Capture"), meta=(BlueprintSpawnableComponent), DisplayName = "Retarget Component")
class URetargetComponent : public UActorComponent
{
	GENERATED_BODY()

public:
	// Sets default values for this component's properties
	UE_API URetargetComponent();

	/**
	* Skeletal Mesh component that will be the source for retargeting. Can be on the Owner Actor or another Actor in the same level.
	*/
	UPROPERTY(BlueprintReadOnly, EditInstanceOnly, Category = "Performance Capture",  
		meta = (UseComponentPicker, AllowAnyActor, AllowedClasses = "/Script/Engine.SkeletalMeshComponent", EditCondition = "IsSourcePerformerNull()", EditConditionHides))
	FComponentReference SourceSkeletalMeshComponent ;

	/**
	* Skeletal Mesh that will be driven by the IKRetargeter. Limited to skeletal meshes on this component's Owner Actor.
	*/
	UPROPERTY(BlueprintReadOnly, EditAnywhere,Category = "Performance Capture", meta = (UseComponentPicker, AllowedClasses = "/Script/Engine.SkeletalMeshComponent"))
	FComponentReference ControlledSkeletalMeshComponent;

	/**
	* Force all skeletal meshes to use the ControlledSkeletalMeshComponent as their Leader. Default = True.
	*/
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Performance Capture")
	bool bForceOtherMeshesToFollowControlledMesh = true;

	/**
	* The IKRetarget Asset to use for retargeting between the source and controlled skeletal meshes.
	*/
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Performance Capture")
	TObjectPtr<UIKRetargeter> RetargetAsset;

	/**
	* Custom Retarget Profile. Should be used to override Retarget settings on the RetargetAsset.
	*/
	UPROPERTY(BlueprintReadOnly, Interp, EditAnywhere, Category = "Performance Capture")
	FRetargetProfile CustomRetargetProfile;

	/**
	* CapturePerformer Actor that will be the source for retargeting.
	*/
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Performance Capture|Character")
	TSoftObjectPtr<ACapturePerformer> SourcePerformer;
	
	/**
	* Set the Source Performer Mesh.
	* @param InPerformerMesh New Source Skeletal Mesh.
	*/
	UFUNCTION(BlueprintCallable, Category = "Performance Capture|Retarget")
	UE_API void SetSourcePerformerMesh(USkeletalMeshComponent* InPerformerMesh);
	
	/**
	* Set the Controlled Skeletal Mesh.
	* @param InControlledMesh New Controlled Skeletal Mesh Component.
	*/
	UFUNCTION(BlueprintCallable, Category = "Performance Capture|Retarget")
	UE_API void SetControlledMesh(USkeletalMeshComponent* InControlledMesh);

	/**
	* Set the Retarget Asset.
	* @param InRetargetAsset New IKRetarget Asset.
	*/
	UFUNCTION(BlueprintCallable, Category = "Performance Capture|Retarget")
	UE_API void SetRetargetAsset(UIKRetargeter* InRetargetAsset);

	/**
	* Set a Custom Retarget Profile.
	* @param InProfile New Retarget Profile.
	*/
	UFUNCTION(BlueprintCallable, Category = "Performance Capture|Retarget")
	UE_API void SetCustomRetargetProfile(FRetargetProfile InProfile);

	/**
	* Get Retarget Profile.
	* @return Current Custom Retarget Profile.
	*/
	UFUNCTION(BlueprintCallable, Category = "Performance Capture|Retarget")
	UE_API FRetargetProfile GetCustomRetargetProfile();

	/**
	* Set the bForceOtherMeshesToFollowControlledMesh variable. Changing will re-initialise animation.
	* @param bInBool Force all other skeletal meshes in the Owner Actor to follow the pose of the Controlled Skeletal Mesh.
	*/
	UFUNCTION(BlueprintCallable, Category = "Performance Capture|Retarget")
	UE_API void SetForceMeshesFollowLeader(bool bInBool);

	/**
	* Set the Source CapturePerformer Actor.
	* @param InPerformer New Performer.
	*/
	UFUNCTION(BlueprintCallable, Category = "Performance Capture|Character")
	UE_API void SetSourcePerformer(ACapturePerformer* InPerformer);
	
protected:
	// Called when the game starts
	UE_API virtual void OnRegister() override; //This is the equivalent of OnConstruction in an actor
	UE_API virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	UE_API virtual void DestroyComponent(bool bPromoteChildren) override;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	UFUNCTION()
	inline bool IsSourcePerformerNull() const { return SourcePerformer.IsNull(); }
	
private:
	bool bIsDirty = true;
public:
	
	UE_API void InitiateAnimation();

	UE_API void SetForceOtherMeshesToFollowControlledMesh(bool bInBool);
};

#undef UE_API
