// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PerformanceCaptureCoreEditor : ModuleRules
{
	public PerformanceCaptureCoreEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
			);
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"IKRig",
				"LiveLinkAnimationCore",
				"LiveLinkInterface",
				"PerformanceCaptureCore"
			}
			);
	}
}
