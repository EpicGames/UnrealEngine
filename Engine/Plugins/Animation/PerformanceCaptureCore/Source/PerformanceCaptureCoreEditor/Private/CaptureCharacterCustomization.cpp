// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureCharacterCustomization.h"

#include "CaptureCharacter.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDetailGroup.h"
#include "PropertyCustomizationHelpers.h"
#include "RetargetComponent.h"

#define LOCTEXT_NAMESPACE "CaptureCharacterCustomization"

TSharedRef<IDetailCustomization> FCaptureCharacterCustomization::MakeInstance()
{
	return MakeShared<FCaptureCharacterCustomization>();
}

void FCaptureCharacterCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> CustomizedObjects;
	DetailBuilder.GetObjectsBeingCustomized(CustomizedObjects);

	// Ensure that we are only customizing one object
	if (CustomizedObjects.Num() != 1)
	{
		return;
	}

	CustomizedCharacter = Cast<ACaptureCharacter>(CustomizedObjects[0]);

	CustomizePerformanceCaptureCategory(DetailBuilder);
}

void FCaptureCharacterCustomization::CustomizePerformanceCaptureCategory(IDetailLayoutBuilder& DetailBuilder)
{
	// Customize the PerformanceCapture category for the Retarget Component property.
	TObjectPtr<URetargetComponent> CustomizedRetargetComponent = CustomizedCharacter->GetRetargetComponent();

	if (CustomizedRetargetComponent == nullptr)
	{
		return;
	}

	IDetailCategoryBuilder& PCapCategory = DetailBuilder.EditCategory("Performance Capture", LOCTEXT("PerformanceCaptureCategoryName", "Performance Capture"));

	const TArray<UObject*> CustomizedRetargetComponentObjects = { CustomizedRetargetComponent };
	PCapCategory.AddExternalObjectProperty(CustomizedRetargetComponentObjects, GET_MEMBER_NAME_CHECKED(URetargetComponent, ControlledSkeletalMeshComponent));
	PCapCategory.AddExternalObjectProperty(CustomizedRetargetComponentObjects, GET_MEMBER_NAME_CHECKED(URetargetComponent, bForceOtherMeshesToFollowControlledMesh));
	PCapCategory.AddExternalObjectProperty(CustomizedRetargetComponentObjects, GET_MEMBER_NAME_CHECKED(URetargetComponent, RetargetAsset));
	PCapCategory.AddExternalObjectProperty(CustomizedRetargetComponentObjects, GET_MEMBER_NAME_CHECKED(URetargetComponent, CustomRetargetProfile));
	PCapCategory.AddExternalObjectProperty(CustomizedRetargetComponentObjects, GET_MEMBER_NAME_CHECKED(URetargetComponent, SourceSkeletalMeshComponent));

	IDetailGroup& CharacterGroup = PCapCategory.AddGroup("Character", LOCTEXT("PerformanceCaptureCharacterGroupName", "Character"));
	CharacterGroup.AddExternalObjectProperty(CustomizedRetargetComponentObjects, GET_MEMBER_NAME_CHECKED(URetargetComponent, SourcePerformer), EPropertyLocation::Default, FAddPropertyParams());
}

#undef LOCTEXT_NAMESPACE
