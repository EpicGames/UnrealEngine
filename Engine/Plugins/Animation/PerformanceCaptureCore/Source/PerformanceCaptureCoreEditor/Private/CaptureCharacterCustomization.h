// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

#include "Styling/SlateTypes.h"
#include "UObject/TemplateString.h"

class ACaptureCharacter;

/**
 * Detail customization for ACaptureCharacter
 */
class FCaptureCharacterCustomization : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	//~ Begin IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization interface */

private:
	void CustomizePerformanceCaptureCategory(IDetailLayoutBuilder& DetailBuilder);

private:
	/** The CaptureCharacter being customized */
	ACaptureCharacter* CustomizedCharacter;
};
