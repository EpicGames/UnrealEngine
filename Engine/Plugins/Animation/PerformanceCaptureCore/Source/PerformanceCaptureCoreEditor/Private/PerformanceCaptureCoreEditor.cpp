// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureCharacter.h"
#include "CaptureCharacterCustomization.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "FPerformanceCaptureCoreEditorModule"

/** Editor module for FPerformanceCaptureCoreEditor */
class FPerformanceCaptureCoreEditorModule : public IModuleInterface
{
	void StartupModule() override
	{
		CaptureCharacterName = ACaptureCharacter::StaticClass()->GetFName();

		FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyEditorModule.RegisterCustomClassLayout(
			ACaptureCharacter::StaticClass()->GetFName(),
			FOnGetDetailCustomizationInstance::CreateStatic(&FCaptureCharacterCustomization::MakeInstance)
		);
	}

	void ShutdownModule() override
	{
		if (FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
		{
			PropertyModule->UnregisterCustomClassLayout(CaptureCharacterName);
		}
	}

	FName CaptureCharacterName;
};

IMPLEMENT_MODULE(FPerformanceCaptureCoreEditorModule, PerformanceCaptureCoreEditor)

#undef LOCTEXT_NAMESPACE
